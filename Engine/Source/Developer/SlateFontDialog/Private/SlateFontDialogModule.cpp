// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFontDialogModule.h"

#include "Modules/ModuleManager.h"
#include "SlateFontDlgWindow.h"

bool FSlateFontDialogModule::OpenFontDialog(FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags)
{
	bool SuccessfulLoad = false;
	FSlateFontDlgWindow FontDialog(SuccessfulLoad);
	if (!SuccessfulLoad)
	{
		return false;
	}

	bool SuccessfulWindow = false;
	FontDialog.OpenFontWindow(OutFontName, OutHeight, OutFlags, SuccessfulWindow);
	return SuccessfulWindow;
}

IMPLEMENT_MODULE(FSlateFontDialogModule, SlateFontDialog);
