// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateFontDlgWindow.h"

#include "Fonts/CompositeFont.h"

#include "Framework/Docking/TabManager.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "SlateOptMacros.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SGridPanel.h"

#include <fontconfig.h>

#define LOCTEXT_NAMESPACE "SlateFontDialogNamespace"

DEFINE_LOG_CATEGORY_STATIC(LogSlateFontDialog, Log, All);

namespace
{
	FcPattern* GetFontMatch(TSharedPtr<FString> SelectedFont, TSharedPtr<FString> SelectedTypeface)
	{
		FcPattern* Query = FcPatternCreate(); 
		FcPatternAddString(Query, FC_FAMILY, reinterpret_cast<const FcChar8*>(TCHAR_TO_UTF8(**SelectedFont)));
		FcPatternAddString(Query, FC_STYLE, reinterpret_cast<const FcChar8*>(TCHAR_TO_UTF8(**SelectedTypeface)));

		// Increase the scope of possible matches
		FcConfigSubstitute(nullptr, Query, FcMatchPattern);
		FcDefaultSubstitute(Query);

		FcResult Result = FcResultNoMatch;
		FcPattern* Match = FcFontMatch(nullptr, Query, &Result);
		if (Result != FcResultMatch)
		{
			UE_LOGF(LogSlateFontDialog, Warning, "FontConfig failed to match a font, error number: %i", static_cast<int>(Result));
			Match = nullptr;
		}

		FcPatternDestroy(Query);

		return Match;
	}

	FString GetFontFilename(TSharedPtr<FString> SelectedFont, TSharedPtr<FString> SelectedTypeface)
	{
		FcPattern* Match = GetFontMatch(SelectedFont, SelectedTypeface);

		FcChar8* FilenameRaw = nullptr;
		FcResult Result = FcPatternGetString(Match, FC_FILE, 0, &FilenameRaw);
		if (Result != FcResultMatch)
		{
			UE_LOGF(LogSlateFontDialog, Warning, "FontConfig failed to match a font, error number: %i", static_cast<int>(Result));
		}
		FString Filename(UTF8_TO_TCHAR(FilenameRaw));

		FcPatternDestroy(Match);

		return Filename;
	}
}

FSlateFontInfo FSlateFontDlgWindow::GetSampleFont() const
{
	TSharedPtr<FCompositeFont> SampleFont = MakeShareable(new FCompositeFont(**SelectedFont, GetFontFilename(SelectedFont, SelectedTypeface), EFontHinting::Default, EFontLoadingPolicy::LazyLoad));
	return FSlateFontInfo(SampleFont, SampleTextSize);
}

void FSlateFontDlgWindow::LoadFonts()
{
	for (int i = 0; i < FontSet->nfont; i++)
	{
		FcPattern* Pattern = FontSet->fonts[i];

		// Grab the font name
		FcChar8* NameRaw = nullptr;
		FcResult Result = FcPatternGetString(Pattern, FC_FAMILY, 0, &NameRaw);
		if (Result != FcResultMatch)
		{
			UE_LOGF(LogSlateFontDialog, Warning, "FontConfig failed to load a font, error number: %i", static_cast<int>(Result));
			continue;
		}
		FString Name(UTF8_TO_TCHAR(NameRaw));

		// Grab the style/typeface name
		FcChar8* StyleRaw = nullptr;
		Result = FcPatternGetString(Pattern, FC_STYLE, 0, &StyleRaw);
		if (Result != FcResultMatch)
		{
			UE_LOGF(LogSlateFontDialog, Warning, "FontConfig failed to load a font, error number: %i", static_cast<int>(Result));
			continue;
		}
		FString Style(UTF8_TO_TCHAR(StyleRaw));
		
		TArray<TSharedPtr<FString>>& Typefaces = TypefaceList.FindOrAdd(Name);
		if (!Typefaces.ContainsByPredicate([&Style] (const TSharedPtr<FString>& Element) { return *Element == Style; }))
		{
			Typefaces.Add(MakeShareable(new FString(Style)));
		}
	}
	
	// Can't use TMap::GenerateKeyArray because we need TSharedPtr<FString>, not FString
	for (const TPair<FString, TArray<TSharedPtr<FString>>>& Typeface : TypefaceList)
	{
		FontList.AddUnique(MakeShareable(new FString(Typeface.Key)));
	}

	FontList.Sort([] (const TSharedPtr<FString>& First, const TSharedPtr<FString>& Second) { return *First < *Second; });

	SelectedFont = FontList[0];
	SelectedTypefaceList = *TypefaceList.Find(*SelectedFont);
	SelectedTypeface = SelectedTypefaceList[0];
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply FSlateFontDlgWindow::OpenColorPicker()
{
	TSharedPtr<SWindow> ColorWindow;
	SAssignNew(ColorWindow, SWindow)
	.Title(LOCTEXT("ColorPickerTest-WindowTitle-StandardColor", "Standard Color"))
	.ClientSize(SColorPicker::DEFAULT_WINDOW_SIZE)
	.IsPopupWindow(true);

	TSharedPtr<SBox> ColorPicker = SNew(SBox)
	.Padding(10.0f)
	[
		SNew(SColorPicker)
		.ParentWindow(ColorWindow)
		.UseAlpha(false)
		.OnColorCommitted_Lambda([this] (const FLinearColor& NewColor) 
		{ 
			FontColor = NewColor;
			FontColor.A = 1.0f;
			SampleTextBlock->SetColorAndOpacity(FontColor); 
		})
	];

	ColorWindow->SetContent(ColorPicker.ToSharedRef());

	FSlateApplication::Get().AddModalWindow(ColorWindow.ToSharedRef(), FGlobalTabmanager::Get()->GetRootWindow());
	return FReply::Handled();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FSlateFontDlgWindow::FSlateFontDlgWindow(bool& OutSuccess)
{
	FontSet = FcConfigGetFonts(nullptr, FcSetSystem);
	if (FontSet->nfont < 1)
	{
		OutSuccess = false;
		UE_LOGF(LogSlateFontDialog, Warning, "FontConfig could not find any fonts");
		return;
	}
	
	OutSuccess = true;
	
	LoadFonts();

	SampleTextStyle = FTextBlockStyle()
	.SetFont(GetSampleFont())
	.SetColorAndOpacity(FontColor);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

// OutFontName actually gets set to the filepath of the chosen font to reduce dependencies in UTrueTypeFontFactory::LoadFontFace within TTFontImport.cpp 
void FSlateFontDlgWindow::OpenFontWindow(FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags, bool& OutSuccess)
{
	OutFontName = GetFontFilename(SelectedFont, SelectedTypeface);
	OutHeight = (float) FontSize;
	OutFlags = EFontImportFlags::None;
	EnumAddFlags(OutFlags, EFontImportFlags::EnableAntialiasing);
	EnumAddFlags(OutFlags, EFontImportFlags::AlphaOnly);

	// Create the font dialogue window
	SAssignNew(Window, SWindow)
	.Title(FText::FromString(TEXT("Fonts")))
	.SizingRule(ESizingRule::Autosized)
	.AutoCenter(EAutoCenter::PreferredWorkArea)
	.SupportsMaximize(false)
	.SupportsMinimize(false)
	.HasCloseButton(false)
	[
		SNew(SGridPanel)

		// Font
		+SGridPanel::Slot(0, 0)
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			// Name
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).Text(FText::FromString(TEXT("Font")))
			]

			// Dropdown
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.WidthOverride(180.0f)
				[
					SNew(STextComboBox)
					.OptionsSource(&FontList)
					.InitiallySelectedItem(FontList[0])
					.OnSelectionChanged_Lambda([this, &OutFontName] (TSharedPtr<FString> InSelection, ESelectInfo::Type InSelectInfo) 
					{
						if (InSelection.IsValid())
						{
							SelectedFont = InSelection;
							SelectedTypefaceList = *TypefaceList.Find(*SelectedFont);
							SelectedTypeface = SelectedTypefaceList[0];
							OutFontName = GetFontFilename(SelectedFont, SelectedTypeface);
							TypefaceDropdown->RefreshOptions();
							TypefaceDropdown->SetSelectedItem(SelectedTypefaceList[0]);

							SampleTextBlock->SetFont(GetSampleFont());
						}
					} )
				]           
			]
		]

		// Typeface
		+SGridPanel::Slot(1, 0)
		.Padding(8.0f)
		[
			SNew(SVerticalBox)
			
			// Name
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).Text(FText::FromString(TEXT("Typeface")))
			]

			// Dropdown
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.WidthOverride(100.0f)
				[
					SAssignNew(TypefaceDropdown, STextComboBox)
					.OptionsSource(&SelectedTypefaceList)
					.InitiallySelectedItem(SelectedTypefaceList[0])
					.OnSelectionChanged_Lambda([this] (TSharedPtr<FString> InSelection, ESelectInfo::Type InSelectInfo) 
					{
						if (InSelection.IsValid())
						{
							SelectedTypeface = InSelection;
							SampleTextBlock->SetFont(GetSampleFont());
						}
					} ) 
				] 
			]
		]

		// Size
		+SGridPanel::Slot(2, 0)
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			// Name
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).Text(FText::FromString(TEXT("Size")))
			]

			// Text box
			+SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				SNew(SNumericEntryBox<uint8>)
				.Value_Lambda( [this] { return FontSize; } ) 
				.OnValueCommitted_Lambda([this, &OutHeight] (float InValue, ETextCommit::Type CommitInfo) 
				{ 
					FontSize = InValue; 
					OutHeight = static_cast<float>(FontSize);
				} )
			]
		]

		// Effects
		+SGridPanel::Slot(0, 1)
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			// Section name
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Effects")))
			]

			// Strikethrough
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Checkbox
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SimplifiedCheckbox"))
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							SampleTextStyle.SetStrikeBrush(*FAppStyle::Get().GetBrush("DefaultTextUnderline"));
						}
						else
						{
							SampleTextStyle.SetStrikeBrush(*FAppStyle::Get().GetBrush("NoBrush"));
						}

						SampleTextBlock->SetTextStyle(&SampleTextStyle);
					})
				]

				// Name
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(STextBlock).Text(FText::FromString(TEXT("Strikethrough")))
				]
			]

			// Underline
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Checkbox
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SimplifiedCheckbox"))
					.OnCheckStateChanged_Lambda([this, &OutFlags](ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							SampleTextStyle.SetUnderlineBrush(*FAppStyle::Get().GetBrush("DefaultTextUnderline"));
							EnumAddFlags(OutFlags, EFontImportFlags::EnableUnderline);
						}
						else
						{
							SampleTextStyle.SetUnderlineBrush(*FAppStyle::Get().GetBrush("NoBrush"));
							EnumRemoveFlags(OutFlags, EFontImportFlags::EnableUnderline);
						}

						SampleTextBlock->SetTextStyle(&SampleTextStyle);
					})
				]

				// Name
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 0.0f)
				[
					SNew(STextBlock).Text(FText::FromString(TEXT("Underline")))
				]
			]
		]

		// Color
		+SGridPanel::Slot(1, 1)
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			// Name
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).Text(FText::FromString(TEXT("Color")))
			]

			// Color picker button
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Menu.Button")
				.OnClicked_Raw(this, &FSlateFontDlgWindow::OpenColorPicker)
				[
					SNew(SOverlay)

					+SOverlay::Slot()
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Bottom)
					[
						SAssignNew(ColorIconText, STextBlock)
						.Text(LOCTEXT("ColorLabel", "A"))
					]

					+SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Bottom)
					[
						SNew(SColorBlock)
						.Color_Lambda([this] () { return FontColor; })
						.Size(FVector2D(20.0f, 6.0f))
					]
				]
			]
		]

		// Sample Text
		+SGridPanel::Slot(2, 1)
		.Padding(8.0f)
		[
			SNew(SBox)
			.MaxDesiredHeight(200)
			.MaxDesiredWidth(600)
			[
				SAssignNew(SampleTextBlock, STextBlock)
				.TextStyle(&SampleTextStyle)
				.Text(FText::FromString(TEXT("Sample Text")))
			]
		]

		// OK / Cancel buttons
		+SGridPanel::Slot(2, 2)
		.Padding(8.0f)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		[
			SNew(SHorizontalBox)

			// OK Button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f)
			[
				SNew(SButton)
				.Text(FText::FromString(TEXT("OK")))
				.OnClicked_Lambda([this, &OutSuccess] () -> FReply 
				{
					OutSuccess = true;
					Window->RequestDestroyWindow();
					return FReply::Handled(); 
				})
			]

			// Cancel Button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f)
			[
				SNew(SButton)
				.Text(FText::FromString(TEXT("Cancel")))
				.OnClicked_Lambda([this, &OutSuccess] () -> FReply 
				{ 
					OutSuccess = false;
					Window->RequestDestroyWindow(); 
					return FReply::Handled(); 
				})
			]
		]
	];

	FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), FGlobalTabmanager::Get()->GetRootWindow()); 
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
