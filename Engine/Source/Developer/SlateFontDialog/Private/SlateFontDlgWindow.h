// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateFontDialogModule.h"

#include "Fonts/SlateFontInfo.h"

#include "Widgets/Input/STextComboBox.h"

#include <fontconfig.h>

#define LOCTEXT_NAMESPACE "SlateFontDialogNamespace"

class FSlateFontDlgWindow
{
private:
	TSharedPtr<SWindow> Window;

	TSharedPtr<FString> SelectedFont;
	TArray<TSharedPtr<FString>> FontList;

	TSharedPtr<FString> SelectedTypeface;
	TArray<TSharedPtr<FString>> SelectedTypefaceList;
	TMap<FString, TArray<TSharedPtr<FString>>> TypefaceList;

	TSharedPtr<STextComboBox> TypefaceDropdown;
	TSharedPtr<STextBlock> SampleTextBlock;
	TSharedPtr<STextBlock> ColorIconText;

	FTextBlockStyle SampleTextStyle;
	const uint8 SampleTextSize = 36;

	uint8 FontSize = SampleTextSize;
	FLinearColor FontColor = FLinearColor::White;

	FcFontSet* FontSet;

	FSlateFontInfo GetSampleFont() const;

	void LoadFonts();

	FReply OpenColorPicker();

public:
	FSlateFontDlgWindow(bool& OutSuccess);

	void OpenFontWindow(FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags, bool& OutSuccess);
};

#undef LOCTEXT_NAMESPACE
