// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "IDesktopPlatform.h"

enum class EFontImportFlags;

class ISlateFontDialogModule : public IModuleInterface
{
public:
	virtual ~ISlateFontDialogModule() = default;

	virtual bool OpenFontDialog(FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags) { return false; }
};
