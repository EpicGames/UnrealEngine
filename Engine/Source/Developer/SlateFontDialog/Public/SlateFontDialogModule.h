// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISlateFontDialogModule.h"

class FSlateFontDialogModule : public ISlateFontDialogModule
{
public:
	bool OpenFontDialog(FString& OutFontName, float& OutHeight, EFontImportFlags& OutFlags);
};
