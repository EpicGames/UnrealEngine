// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Linux")]
public class SlateFontDialog : ModuleRules
{
	public SlateFontDialog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"AppFramework",
				"InputCore",
				"Slate",
				"SlateCore"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"DesktopPlatform"
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, new string[]
		{
			"FreeType2",
			"FontConfig"
		});
	}
}
